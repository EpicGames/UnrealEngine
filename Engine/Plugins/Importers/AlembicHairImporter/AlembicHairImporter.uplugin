{
	"FileVersion": 1,
	"Version": 2,
	"VersionName": "1.0",
	"FriendlyName": "Alembic Groom Importer",
	"Description": "Import Hair Strands from Alembic file",
	"Category": "Geometry",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AlembicHairTranslatorModule",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "HairStrands",
			"Enabled": true
		}
	]
}
