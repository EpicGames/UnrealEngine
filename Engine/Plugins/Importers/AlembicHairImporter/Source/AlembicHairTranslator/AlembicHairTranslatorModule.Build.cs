// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AlembicHairTranslatorModule : ModuleRules
	{
        public AlembicHairTranslatorModule(ReadOnlyTargetRules Target) : base(Target)
		{
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
					"Imath",
                    "AlembicLib",
                    "Core",
                    "Engine",
                    "HairStrandsCore",
                    "HairStrandsEditor",
					"CoreUObject"
                });
        }
	}
}
