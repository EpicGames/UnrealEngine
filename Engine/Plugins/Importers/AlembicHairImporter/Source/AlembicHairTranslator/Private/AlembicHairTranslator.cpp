// Copyright Epic Games, Inc. All Rights Reserved.

#include "AlembicHairTranslator.h"

#include "GroomCacheData.h"
#include "HairAttributes.h"
#include "HairDescription.h"
#include "GroomSettings.h"
#include "Math/RotationMatrix.h"
#include "Misc/Paths.h"
#include "Math/ScaleMatrix.h"

#if PLATFORM_WINDOWS
#include "HairAttributes.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
#include "Alembic/AbcGeom/All.h"
#include "Alembic/AbcCoreFactory/IFactory.h"
#include "Alembic/Abc/IArchive.h"
#include "Alembic/Abc/IObject.h"
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogAlembicHairImporter, Log, All);

namespace AlembicHairFormat
{
	static const float RootRadius = 0.0001f; // m
	static const float TipRadius = 0.00005f; // m

	static constexpr const float UNIT_TO_CM = 1;
}

namespace AlembicHairTranslatorUtils
{
	bool IsAttributeValid(const FString& AttributeName)
	{
		// Ignore the attributes that aren't prefixed with groom_
		return AttributeName.StartsWith(TEXT("groom_"));
	}

	// Groom attributes from UserProperties (as ScalarProperty)
	template <typename AbcParamType, typename AttributeType>
	void SetGroomAttributes(FHairDescription& HairDescription, const Alembic::AbcGeom::ICompoundProperty& Parameters, const std::string& PropName)
	{
		FName AttributeName(ANSI_TO_TCHAR(PropName.c_str()));

		AbcParamType Param(Parameters, PropName);
		AttributeType ParamValue = Param.getValue();
		SetGroomAttribute(HairDescription, FGroomID(0), AttributeName, ParamValue);
	}

	template <>
	void SetGroomAttributes<Alembic::Abc::IStringProperty, FName>(FHairDescription& HairDescription, const Alembic::AbcGeom::ICompoundProperty& Parameters, const std::string& PropName)
	{
		FName AttributeName(ANSI_TO_TCHAR(PropName.c_str()));

		Alembic::Abc::IStringProperty Param(Parameters, PropName);
		FName ParamValue = FName(Param.getValue().c_str());
		SetGroomAttribute(HairDescription, FGroomID(0), AttributeName, ParamValue);
	}

	// Groom attributes from arbitrary GeomParams (as ArrayProperty with only one value)
	template <typename AbcParamType, typename AbcArraySampleType, typename AttributeType>
	void SetGroomAttributes(FHairDescription& HairDescription, const Alembic::AbcGeom::ICompoundProperty& Parameters, const std::string& PropName)
	{
		FName AttributeName(ANSI_TO_TCHAR(PropName.c_str()));

		AbcParamType Param(Parameters, PropName);
		AbcArraySampleType ParamValues = Param.getExpandedValue().getVals();
		if (ParamValues->size() == 1)
		{
			AttributeType ParamValue = (*ParamValues)[0];
			SetGroomAttribute(HairDescription, FGroomID(0), AttributeName, ParamValue);
		}
	}

	template <>
	void SetGroomAttributes<Alembic::AbcGeom::IStringGeomParam, Alembic::Abc::StringArraySamplePtr, FName>(FHairDescription& HairDescription, const Alembic::AbcGeom::ICompoundProperty& Parameters, const std::string& PropName)
	{
		FName AttributeName(ANSI_TO_TCHAR(PropName.c_str()));

		Alembic::AbcGeom::IStringGeomParam Param(Parameters, PropName);
		Alembic::Abc::StringArraySamplePtr ParamValues = Param.getExpandedValue().getVals();
		if (ParamValues->size() == 1)
		{
			FName ParamValue = FName((*ParamValues)[0].c_str());
			SetGroomAttribute(HairDescription, FGroomID(0), AttributeName, ParamValue);
		}
	}

	template <typename AbcParamType, typename AbcArraySampleType, typename AttributeType>
	void SetGroomAttributes(FHairDescription& HairDescription, const Alembic::AbcGeom::ICompoundProperty& Parameters, const std::string& PropName, uint8 Extent)
	{
		FName AttributeName(ANSI_TO_TCHAR(PropName.c_str()));

		AbcParamType Param(Parameters, PropName);
		AbcArraySampleType ParamValues = Param.getExpandedValue().getVals();
		if (ParamValues->size() == 1)
		{
			AttributeType ParamValue = {};
			for (int32 Index = 0; Index < Extent; ++Index)
			{
				ParamValue[Index] = (*ParamValues)[0][Index];
			}
			SetGroomAttribute(HairDescription, FGroomID(0), AttributeName, ParamValue);
		}
	}

	template <typename AttributeType>
	void SetGroomAttribute(FHairDescription& HairDescription, FName AttributeName, AttributeType AttributeValue)
	{
		SetGroomAttribute(HairDescription, FGroomID(0), AttributeName, AttributeValue);
	}

	void SetGroomAttributes(FHairDescription& HairDescription, const Alembic::AbcGeom::ICompoundProperty& Parameters)
	{
		for (int Index = 0; Index < Parameters.getNumProperties(); ++Index)
		{
			Alembic::Abc::PropertyHeader PropertyHeader = Parameters.getPropertyHeader(Index);
			std::string PropName = PropertyHeader.getName();

			if (!IsAttributeValid(ANSI_TO_TCHAR(PropName.c_str())))
			{
				continue;
			}

			Alembic::Abc::PropertyType PropType = PropertyHeader.getPropertyType();

			if (PropType != Alembic::Abc::kCompoundProperty)
			{
				Alembic::Abc::DataType DataType = PropertyHeader.getDataType();
				uint8 Extent = DataType.getExtent();

				switch (DataType.getPod())
				{
				case Alembic::Util::kInt16POD:
				{
					if (PropType == Alembic::Abc::kScalarProperty)
					{
						SetGroomAttributes<Alembic::Abc::IInt16Property, int>(HairDescription, Parameters, PropName);
					}
					else
					{
						SetGroomAttributes<Alembic::AbcGeom::IInt16GeomParam, Alembic::Abc::Int16ArraySamplePtr, int>(HairDescription, Parameters, PropName);
					}
				}
				break;
				case Alembic::Util::kInt32POD:
				{
					if (PropType == Alembic::Abc::kScalarProperty)
					{
						SetGroomAttributes<Alembic::Abc::IInt32Property, int>(HairDescription, Parameters, PropName);
					}
					else
					{
						SetGroomAttributes<Alembic::AbcGeom::IInt32GeomParam, Alembic::Abc::Int32ArraySamplePtr, int>(HairDescription, Parameters, PropName);
					}
				}
				break;
				case Alembic::Util::kStringPOD:
				{
					if (PropType == Alembic::Abc::kScalarProperty)
					{
						SetGroomAttributes<Alembic::Abc::IStringProperty, FName>(HairDescription, Parameters, PropName);
					}
					else
					{
						SetGroomAttributes<Alembic::AbcGeom::IStringGeomParam, Alembic::Abc::StringArraySamplePtr, FName>(HairDescription, Parameters, PropName);
					}
				}
				break;
				case Alembic::Util::kFloat32POD:
				{
					switch (Extent)
					{
					case 1:
						SetGroomAttributes<Alembic::AbcGeom::IFloatGeomParam, Alembic::Abc::FloatArraySamplePtr, float>(HairDescription, Parameters, PropName);
						break;
					case 2:
						SetGroomAttributes<Alembic::AbcGeom::IV2fGeomParam, Alembic::Abc::V2fArraySamplePtr, FVector2f>(HairDescription, Parameters, PropName, Extent);
						break;
					case 3:
						SetGroomAttributes<Alembic::AbcGeom::IV3fGeomParam, Alembic::Abc::V3fArraySamplePtr, FVector3f>(HairDescription, Parameters, PropName, Extent);
						break;
					}
				}
				break;
				case Alembic::Util::kFloat64POD:
				{
					switch (Extent)
					{
					case 1:
						SetGroomAttributes<Alembic::AbcGeom::IDoubleGeomParam, Alembic::Abc::DoubleArraySamplePtr, float>(HairDescription, Parameters, PropName);
						break;
					case 2:
						SetGroomAttributes<Alembic::AbcGeom::IV2dGeomParam, Alembic::Abc::V2dArraySamplePtr, FVector2f>(HairDescription, Parameters, PropName, Extent);
						break;
					case 3:
						SetGroomAttributes<Alembic::AbcGeom::IV3dGeomParam, Alembic::Abc::V3dArraySamplePtr, FVector3f>(HairDescription, Parameters, PropName, Extent);
						break;
					}
				}
				break;
				}
			}
		}
	}

	// Default conversion traits
	template<typename TAttributeType>
	struct TConvertor
	{
		template<typename InType>
		static TAttributeType ConvertType(const InType& In)
		{
			return In;
		}
	};
	// String conversion traits
	template<>
	struct TConvertor<FName>
	{
		template<typename InType>
		static FName ConvertType(const InType& In)
		{
			return FName(ANSI_TO_TCHAR(In.c_str()));
		}
	};

	EGroomBasisType MapAlembicEnumToUe(Alembic::AbcGeom::BasisType AlembicBasisType, bool& bSuccess)
	{
		bSuccess = true;
		switch (AlembicBasisType)
		{
		case Alembic::AbcGeom::kNoBasis:
			return EGroomBasisType::NoBasis;
		case Alembic::AbcGeom::kBezierBasis:
			return EGroomBasisType::BezierBasis;
		case Alembic::AbcGeom::kBsplineBasis:
			return EGroomBasisType::BsplineBasis;
		case Alembic::AbcGeom::kCatmullromBasis:
			return EGroomBasisType::CatmullromBasis;
		case Alembic::AbcGeom::kHermiteBasis:
			return EGroomBasisType::HermiteBasis;
		case Alembic::AbcGeom::kPowerBasis:
			return EGroomBasisType::PowerBasis;
		default:
			ensureMsgf(false, TEXT("Unsupported basis type: %d"), AlembicBasisType);
			break;
		}

		bSuccess = false;
		return EGroomBasisType::NoBasis;
	}

	EGroomCurveType MapAlembicEnumToUe(Alembic::AbcGeom::CurveType AlembicCurveType, bool& bSuccess)
	{
		bSuccess = true;
		switch (AlembicCurveType)
		{
		case Alembic::AbcGeom::kCubic:
			return EGroomCurveType::Cubic;
		case Alembic::AbcGeom::kLinear:
			return EGroomCurveType::Linear;
		case Alembic::AbcGeom::kVariableOrder:
			return EGroomCurveType::VariableOrder;
		default:
			ensureMsgf(false, TEXT("Unsupported curve type: %d"), AlembicCurveType);
			break;
		}

		bSuccess = false;
		return EGroomCurveType::Linear;
	}

	template<typename AlembicEnumType>
	bool ConvertAlembicEnumToStrandAttribute(FHairDescription& HairDescription, FStrandID StrandID, AlembicEnumType AlembicEnumValue, FName AttributeName)
	{
		bool bSuccess = false;
		auto UeValue = MapAlembicEnumToUe(AlembicEnumValue, bSuccess);
		if (bSuccess)
		{
			UEnum* UeEnum = StaticEnum<decltype(UeValue)>();
			if (ensure(UeEnum))
			{
				FName ValueName = *UeEnum->GetNameStringByValue((int)UeValue);
				if (ValueName.IsNone())
				{
					bSuccess = false;
				}
				else
				{
					SetHairStrandAttribute(HairDescription, StrandID, AttributeName, ValueName);
				}
			}
			else
			{
				bSuccess = false;
			}
		}
		return bSuccess;
	}

	template <typename AbcParamType, typename AbcArraySampleType, typename AttributeType>
	void ConvertAlembicAttribute(FHairDescription& HairDescription, int32 StartStrandID, int32 NumStrands, int32 StartVertexID, int32 NumVertices, const Alembic::AbcGeom::ICompoundProperty& Parameters, const std::string& PropName)
	{
		FName AttributeName(ANSI_TO_TCHAR(PropName.c_str()));

		// Get the param scope and check if it's supported, otherwise fall back to using the number of values in Param to deduce its scope
		AbcParamType Param(Parameters, PropName);
		AbcArraySampleType ParamValues = Param.getExpandedValue().getVals();
		Alembic::AbcGeom::GeometryScope Scope = Param.getScope();
		int32 NumValues = ParamValues->size();

		// Check the supported scope: UniformScope or VertexScope (1.2), ConstantScope (1.3)
		if (Scope == Alembic::AbcGeom::kUniformScope || Scope == Alembic::AbcGeom::kConstantScope || NumValues == NumStrands)
		{
			TStrandAttributesRef<AttributeType> StrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<AttributeType>(AttributeName);
			if (!StrandAttributeRef.IsValid())
			{
				HairDescription.StrandAttributes().RegisterAttribute<AttributeType>(AttributeName);
				StrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<AttributeType>(AttributeName);
			}

			// Additional check in case the scope is wrongly set as ConstantScope but the number of values match the number of strands
			// Treat that case as UniformScope. Note that with ConstantScope, NumValues is 1, thus the reason to iterate on NumStrands
			bool bIsConstantValue = Scope == Alembic::AbcGeom::kConstantScope && NumValues != NumStrands;
			for (int32 StrandIndex = 0; StrandIndex < NumStrands; ++StrandIndex)
			{
				StrandAttributeRef[FStrandID(StartStrandID + StrandIndex)] = TConvertor<AttributeType>::ConvertType((*ParamValues)[bIsConstantValue ? 0 : StrandIndex]);
			}
		}
		else if (Scope == Alembic::AbcGeom::kVertexScope || NumValues == NumVertices)
		{
			TVertexAttributesRef<AttributeType> VertexAttributeRef = HairDescription.VertexAttributes().GetAttributesRef<AttributeType>(AttributeName);
			if (!VertexAttributeRef.IsValid())
			{
				HairDescription.VertexAttributes().RegisterAttribute<AttributeType>(AttributeName);
				VertexAttributeRef = HairDescription.VertexAttributes().GetAttributesRef<AttributeType>(AttributeName);
			}

			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				VertexAttributeRef[FVertexID(StartVertexID + VertexIndex)] = TConvertor<AttributeType>::ConvertType((*ParamValues)[VertexIndex]);
			}
		}
	}

	template <typename AbcParamType, typename AbcArraySampleType, typename AttributeType>
	void ConvertAlembicAttribute(FHairDescription& HairDescription, int32 StartStrandID, int32 NumStrands, int32 StartVertexID, int32 NumVertices, const Alembic::AbcGeom::ICompoundProperty& Parameters, const std::string& PropName, uint8 Extent)
	{
		FName AttributeName(ANSI_TO_TCHAR(PropName.c_str()));

		// Get the param scope and check if it's supported, otherwise fall back to using the number of values in Param to deduce its scope
		AbcParamType Param(Parameters, PropName);
		AbcArraySampleType ParamValues = Param.getExpandedValue().getVals();
		Alembic::AbcGeom::GeometryScope Scope = Param.getScope();
		int32 NumValues = ParamValues->size();

		// Check the supported scope: UniformScope or VertexScope (1.2), ConstantScope (1.3)
		if (Scope == Alembic::AbcGeom::kUniformScope || Scope == Alembic::AbcGeom::kConstantScope || NumValues == NumStrands)
		{
			TStrandAttributesRef<AttributeType> StrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<AttributeType>(AttributeName);
			if (!StrandAttributeRef.IsValid())
			{
				HairDescription.StrandAttributes().RegisterAttribute<AttributeType>(AttributeName, 1, AttributeType::ZeroVector);
				StrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<AttributeType>(AttributeName);
			}

			// Additional check in case the scope is wrongly set as ConstantScope but the number of values match the number of strands
			// Treat that case as UniformScope. Note that with ConstantScope, NumValues is 1, thus the reason to iterate on NumStrands
			bool bIsConstantValue = Scope == Alembic::AbcGeom::kConstantScope && NumValues != NumStrands;
			for (int32 StrandIndex = 0; StrandIndex < NumStrands; ++StrandIndex)
			{
				AttributeType ParamValue = {};
				for (int32 Index = 0; Index < Extent; ++Index)
				{
					ParamValue[Index] = (*ParamValues)[bIsConstantValue ? 0 : StrandIndex][Index];
				}

				StrandAttributeRef[FStrandID(StartStrandID + StrandIndex)] = ParamValue;
			}
		}
		else if (Scope == Alembic::AbcGeom::kVertexScope || NumValues == NumVertices)
		{
			TVertexAttributesRef<AttributeType> VertexAttributeRef = HairDescription.VertexAttributes().GetAttributesRef<AttributeType>(AttributeName);
			if (!VertexAttributeRef.IsValid())
			{
				HairDescription.VertexAttributes().RegisterAttribute<AttributeType>(AttributeName, 1, AttributeType::ZeroVector);
				VertexAttributeRef = HairDescription.VertexAttributes().GetAttributesRef<AttributeType>(AttributeName);
			}

			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				AttributeType ParamValue = {};
				for (int32 Index = 0; Index < Extent; ++Index)
				{
					ParamValue[Index] = (*ParamValues)[VertexIndex][Index];
				}

				VertexAttributeRef[FVertexID(StartVertexID + VertexIndex)] = ParamValue;
			}
		}
	}

	void ConvertIndexedStringAttribute(FHairDescription& HairDescription, int32 StartStrandID, int32 NumStrands, const Alembic::AbcGeom::ICompoundProperty& CompoundProperty, const std::string& PropName)
	{
		// The compound property should contain 2 properties: ".vals" and ".indices"
		if (CompoundProperty.getNumProperties() != 2)
		{
			return;
		}

		// First property ".vals" should contain the string values
		TArray<FName> IndexedValues;
		{
			Alembic::Abc::PropertyHeader PropertyHeader = CompoundProperty.getPropertyHeader(0);
			Alembic::Abc::DataType DataType = PropertyHeader.getDataType();

			// Basic property validation: array of single strings
			if (PropertyHeader.getPropertyType() != Alembic::Abc::kArrayProperty || 
				DataType.getPod() != Alembic::Util::kStringPOD ||
				DataType.getExtent() != 1)
			{
				return;
			}

			// The strings are converted and stored temporarily as FName
			Alembic::AbcGeom::IStringGeomParam Param(CompoundProperty, PropertyHeader.getName());
			Alembic::Abc::StringArraySamplePtr ParamValues = Param.getExpandedValue().getVals();
			const int32 NumValues = ParamValues->size();

			IndexedValues.SetNum(NumValues);
			for (int32 ValIndex = 0; ValIndex < NumValues; ++ValIndex)
			{
				IndexedValues[ValIndex] = TConvertor<FName>::ConvertType((*ParamValues)[ValIndex]);
			}
		}

		// Second property ".indices" should contain indices for each strand that map to the string values
		{
			Alembic::Abc::PropertyHeader PropertyHeader = CompoundProperty.getPropertyHeader(1);
			Alembic::Abc::DataType DataType = PropertyHeader.getDataType();

			// Basic property validation: array of single uint32 values
			if (PropertyHeader.getPropertyType() != Alembic::Abc::kArrayProperty ||
				DataType.getPod() != Alembic::Util::kUint32POD ||
				DataType.getExtent() != 1)
			{
				return;
			}

			Alembic::AbcGeom::IUInt32GeomParam Param(CompoundProperty, PropertyHeader.getName());
			Alembic::Abc::UInt32ArraySamplePtr ParamValues = Param.getExpandedValue().getVals();
			const int32 NumValues = ParamValues->size();

			// Must have an index value for each strand
			if (NumValues != NumStrands)
			{
				return;
			}

			FName AttributeName(ANSI_TO_TCHAR(PropName.c_str()));
			TStrandAttributesRef<FName> StrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<FName>(AttributeName);
			if (!StrandAttributeRef.IsValid())
			{
				HairDescription.StrandAttributes().RegisterAttribute<FName>(AttributeName);
				StrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<FName>(AttributeName);
			}

			// The indices are converted to FName and stored in the HairDescription
			for (int32 StrandIndex = 0; StrandIndex < NumStrands; ++StrandIndex)
			{
				uint32 ValueIndex = TConvertor<uint32>::ConvertType((*ParamValues)[StrandIndex]);
				if (IndexedValues.IsValidIndex(ValueIndex))
				{
					StrandAttributeRef[FStrandID(StartStrandID + StrandIndex)] = IndexedValues[ValueIndex];
				}
			}
		}
	}

	/** Convert the given Alembic parameters to hair attributes in the proper scope */
	void ConvertAlembicAttributes(FHairDescription& HairDescription, int32 StartStrandID, int32 NumStrands, int32 StartVertexID, int32 NumVertices, const Alembic::AbcGeom::ICompoundProperty& Parameters, FGroomAnimationInfo* AnimInfo)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ConvertAlembicAttributes);

		for (int Index = 0; Index < Parameters.getNumProperties(); ++Index)
		{
			Alembic::Abc::PropertyHeader PropertyHeader = Parameters.getPropertyHeader(Index);
			const std::string PropName = PropertyHeader.getName();
			const FString AttributeName = ANSI_TO_TCHAR(PropName.c_str());

			if (!IsAttributeValid(AttributeName))
			{
				continue;
			}

			Alembic::Abc::PropertyType PropType = PropertyHeader.getPropertyType();

			// Only process ArrayProperty as ITypedGeomParam are not compatible with ScalarProperty
			// ScalarProperty should only be used for groom-scope properties
			if (PropType == Alembic::Abc::kArrayProperty)
			{
				Alembic::AbcCoreAbstract::TimeSamplingPtr TimeSampler = PropertyHeader.getTimeSampling();

				// Check if the groom_color attribute is animated
				if (AnimInfo && AttributeName == TEXT("groom_color") && TimeSampler)
				{
					 const int32 NumSamples = TimeSampler->getNumStoredTimes();
					 if (NumSamples > 1)
					 {
						 const float MinTime = (float)TimeSampler->getSampleTime(0);
						 const float MaxTime = (float)TimeSampler->getSampleTime(NumSamples - 1);
						 AnimInfo->StartTime = FMath::Min(AnimInfo->StartTime, MinTime);
						 AnimInfo->EndTime = FMath::Max(AnimInfo->EndTime, MaxTime);
						 AnimInfo->Attributes |= EGroomCacheAttributes::Color;
					 }
				}

				Alembic::Abc::DataType DataType = PropertyHeader.getDataType();
				uint8 Extent = DataType.getExtent();

				switch (DataType.getPod())
				{
				case Alembic::Util::kStringPOD:
				{
					ConvertAlembicAttribute<Alembic::AbcGeom::IStringGeomParam, Alembic::Abc::StringArraySamplePtr, FName>(HairDescription, StartStrandID, NumStrands, StartVertexID, NumVertices, Parameters, PropName);
				}
				break;
				case Alembic::Util::kBooleanPOD:
				{
					ConvertAlembicAttribute<Alembic::AbcGeom::IBoolGeomParam, Alembic::Abc::BoolArraySamplePtr, bool>(HairDescription, StartStrandID, NumStrands, StartVertexID, NumVertices, Parameters, PropName);
				}
				break;
				case Alembic::Util::kInt8POD:
				{
					ConvertAlembicAttribute<Alembic::AbcGeom::ICharGeomParam, Alembic::Abc::CharArraySamplePtr, int>(HairDescription, StartStrandID, NumStrands, StartVertexID, NumVertices, Parameters, PropName);
				}
				break;
				case Alembic::Util::kInt16POD:
				{
					ConvertAlembicAttribute<Alembic::AbcGeom::IInt16GeomParam, Alembic::Abc::Int16ArraySamplePtr, int>(HairDescription, StartStrandID, NumStrands, StartVertexID, NumVertices, Parameters, PropName);
				}
				break;
				case Alembic::Util::kInt32POD:
				{
					switch (Extent)
					{
					case 1:
						ConvertAlembicAttribute<Alembic::AbcGeom::IInt32GeomParam, Alembic::Abc::Int32ArraySamplePtr, int>(HairDescription, StartStrandID, NumStrands, StartVertexID, NumVertices, Parameters, PropName);
						break;
					case 3:
						ConvertAlembicAttribute<Alembic::AbcGeom::IV3iGeomParam, Alembic::Abc::V3iArraySamplePtr, FVector3f>(HairDescription, StartStrandID, NumStrands, StartVertexID, NumVertices, Parameters, PropName, DataType.getExtent());
						break;
					}
				}
				break;
				case Alembic::Util::kFloat32POD:
				{
					switch (Extent)
					{
					case 1:
						ConvertAlembicAttribute<Alembic::AbcGeom::IFloatGeomParam, Alembic::Abc::FloatArraySamplePtr, float>(HairDescription, StartStrandID, NumStrands, StartVertexID, NumVertices, Parameters, PropName);
						break;
					case 2:
						ConvertAlembicAttribute<Alembic::AbcGeom::IV2fGeomParam, Alembic::Abc::V2fArraySamplePtr, FVector2f>(HairDescription, StartStrandID, NumStrands, StartVertexID, NumVertices, Parameters, PropName, DataType.getExtent());
						break;
					case 3:
						ConvertAlembicAttribute<Alembic::AbcGeom::IV3fGeomParam, Alembic::Abc::V3fArraySamplePtr, FVector3f>(HairDescription, StartStrandID, NumStrands, StartVertexID, NumVertices, Parameters, PropName, DataType.getExtent());
						break;
					}
				}
				break;
				case Alembic::Util::kFloat64POD:
				{
					switch (Extent)
					{
					case 1:
						ConvertAlembicAttribute<Alembic::AbcGeom::IDoubleGeomParam, Alembic::Abc::DoubleArraySamplePtr, float>(HairDescription, StartStrandID, NumStrands, StartVertexID, NumVertices, Parameters, PropName);
						break;
					case 2:
						ConvertAlembicAttribute<Alembic::AbcGeom::IV2dGeomParam, Alembic::Abc::V2dArraySamplePtr, FVector2f>(HairDescription, StartStrandID, NumStrands, StartVertexID, NumVertices, Parameters, PropName, DataType.getExtent());
						break;
					case 3:
						ConvertAlembicAttribute<Alembic::AbcGeom::IV3dGeomParam, Alembic::Abc::V3dArraySamplePtr, FVector3f>(HairDescription, StartStrandID, NumStrands, StartVertexID, NumVertices, Parameters, PropName, DataType.getExtent());
						break;
					}
				}
				break;
				}
			}
			else if (PropType == Alembic::Abc::kCompoundProperty)
			{
				// Special case for indexed string attribute
				Alembic::AbcGeom::ICompoundProperty CompoundProperty(Parameters, PropName);
				ConvertIndexedStringAttribute(HairDescription, StartStrandID, NumStrands, CompoundProperty, PropName);
			}
		}
	}
}

FMatrix ConvertAlembicMatrix(const Alembic::Abc::M44d& AbcMatrix)
{
	FMatrix Matrix;
	for (uint32 i = 0; i < 16; ++i)
	{
		Matrix.M[i >> 2][i % 4] = (float)AbcMatrix.getValue()[i];
	}

	return Matrix;
}

static void ParseObject(const Alembic::Abc::IObject& InObject, float FrameTime, FHairDescription& HairDescription, const FMatrix& ParentMatrix, const FMatrix& ConversionMatrix, float Scale, bool bCheckGroomAttributes, FGroomAnimationInfo* AnimInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ParseObject);

	// Get MetaData info from current Alembic Object
	const Alembic::Abc::MetaData ObjectMetaData = InObject.getMetaData();
	const uint32 NumChildren = InObject.getNumChildren();

	FMatrix LocalMatrix = ParentMatrix;

	bool bHandled = false;
	if (Alembic::AbcGeom::ICurves::matches(ObjectMetaData))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ParseICurves);

		Alembic::AbcGeom::ICurves Curves = Alembic::AbcGeom::ICurves(InObject, Alembic::Abc::kWrapExisting);

		if (AnimInfo)
		{
			// Collect info about the animated attributes of the groom 
			const Alembic::AbcGeom::ICurvesSchema& Schema = Curves.getSchema();

			const bool bIsPositionAnimated = !Schema.isConstant();
			const bool bIsWidthAnimated = Schema.getWidthsParam().valid() && !Schema.getWidthsParam().isConstant();
			if (bIsPositionAnimated)
			{
				AnimInfo->Attributes |= EGroomCacheAttributes::Position;
			}
			if (bIsWidthAnimated)
			{
				AnimInfo->Attributes |= EGroomCacheAttributes::Width;
			}

			uint32 NumSamples = Schema.getNumSamples();
			AnimInfo->NumFrames = FMath::Max(AnimInfo->NumFrames, NumSamples);

			Alembic::AbcCoreAbstract::TimeSamplingPtr TimeSampler = Schema.getTimeSampling();
			if (TimeSampler)
			{
				const float MinTime = (float)TimeSampler->getSampleTime(0);
				const float MaxTime = (float)TimeSampler->getSampleTime(NumSamples - 1);
				AnimInfo->StartTime = FMath::Min(AnimInfo->StartTime, MinTime);
				AnimInfo->EndTime = FMath::Max(AnimInfo->EndTime, MaxTime);

				// We know the seconds per frame, so if we take the time for the first stored sample we can work out how many 'empty' frames come before it
				// Ensure that the start frame is never lower than 0
				Alembic::AbcCoreAbstract::TimeSamplingType SamplingType = TimeSampler->getTimeSamplingType();
				float TimeStep = (float) SamplingType.getTimePerCycle();
				if (TimeStep > 0.f)
				{
					// Set the value in case it couldn't be retrieved from the archive
					if (AnimInfo->SecondsPerFrame == 0.f)
					{
						AnimInfo->SecondsPerFrame = TimeStep;
					}

					const int32 StartFrame = FMath::Max<int32>(FMath::CeilToInt(MinTime / TimeStep), 0);
					const int32 EndFrame = FMath::Max<int32>(FMath::CeilToInt(MaxTime / TimeStep), 0);

					AnimInfo->StartFrame = FMath::Min(AnimInfo->StartFrame, StartFrame);
					AnimInfo->EndFrame = FMath::Max(AnimInfo->EndFrame, EndFrame);
				}
			}
		}

		Alembic::Abc::ISampleSelector SampleSelector((Alembic::Abc::chrono_t) FrameTime);
		Alembic::AbcGeom::ICurves::schema_type::Sample Sample = Curves.getSchema().getValue(SampleSelector);

		Alembic::Abc::FloatArraySamplePtr Widths = Curves.getSchema().getWidthsParam() ? Curves.getSchema().getWidthsParam().getExpandedValue(SampleSelector).getVals() : nullptr;
		Alembic::Abc::P3fArraySamplePtr Positions = Sample.getPositions();
		Alembic::Abc::Int32ArraySamplePtr NumVertices = Sample.getCurvesNumVertices();
		Alembic::Abc::FloatArraySamplePtr Knots = Sample.getKnots();
		Alembic::Abc::UcharArraySamplePtr Orders = Sample.getOrders();

		Alembic::AbcGeom::BasisType BasisType = Sample.getBasis();
		Alembic::AbcGeom::CurveType CurveType = Sample.getType();

		const int32 NumWidths = Widths ? Widths->size() : 0;
		uint32 NumPoints = Positions ? Positions->size() : 0;
		uint32 NumCurves = NumVertices ? NumVertices->size() : 0; // equivalent to Sample.getNumCurves()
		const uint32 NumKnots = Knots ? Knots->size() : 0;

		// Get the starting strand and vertex IDs for this group of ICurves
		int32 StartStrandID = HairDescription.GetNumStrands();
		int32 StartVertexID = HairDescription.GetNumVertices();

		FMatrix ConvertedMatrix = ParentMatrix * ConversionMatrix;
		uint32 GlobalIndex = 0;
		uint32 TotalVertices = 0;
		for (uint32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
		{
			const uint32 CurveNumVertices = (*NumVertices)[CurveIndex];

			// Check the running total number of vertices and skip processing the rest of the node if there is mismatch with the number of points
			TotalVertices += CurveNumVertices;
			if (TotalVertices > NumPoints)
			{
				UE_LOGF(LogAlembicHairImporter, Warning, "Curve %u of %u has %u vertices which causes total vertices (%u) to exceed the expected vertices (%u) in ICurves node. This curve and the remaining ones in the node will be skipped.",
					CurveIndex + 1, NumCurves, CurveNumVertices, TotalVertices, NumPoints);

				// Adjust the number of curves and points to those processed so far
				NumCurves = HairDescription.GetNumStrands() - StartStrandID;
				NumPoints = HairDescription.GetNumVertices() - StartVertexID;
				break;
			}

			FStrandID StrandID = HairDescription.AddStrand();

			SetHairStrandAttribute(HairDescription, StrandID, HairAttribute::Strand::VertexCount, (int) CurveNumVertices);

			AlembicHairTranslatorUtils::ConvertAlembicEnumToStrandAttribute(HairDescription, StrandID, BasisType, HairAttribute::Strand::BasisType);
			AlembicHairTranslatorUtils::ConvertAlembicEnumToStrandAttribute(HairDescription, StrandID, CurveType, HairAttribute::Strand::CurveType);

			for (uint32 PointIndex = 0; PointIndex < CurveNumVertices; ++PointIndex, ++GlobalIndex)
			{
				FVertexID VertexID = HairDescription.AddVertex();

				Alembic::Abc::P3fArraySample::value_type Position = (*Positions)[GlobalIndex];

				FVector3f ConvertedPosition = (FVector4f)ConvertedMatrix.TransformPosition(FVector(Position.x, Position.y, Position.z));
				SetHairVertexAttribute(HairDescription, VertexID, HairAttribute::Vertex::Position, ConvertedPosition);
			}
		}

		// Extract RootUV attribute from ICurves UVsParam
		const bool bHasUVsParam = Curves.getSchema().getUVsParam().valid();
		if (bHasUVsParam)
		{
			Alembic::AbcGeom::IV2fGeomParam UVsParam = Curves.getSchema().getUVsParam();
			Alembic::Abc::V2fArraySamplePtr UVs = UVsParam.getExpandedValue(SampleSelector).getVals();
			auto ConvertUVParam = [UVs](int32 Index)
			{
				// Convert IV2fGeomParam value to FVector2f by their individual components
				FVector2f ParamValue;
				for (int32 ExtentIndex = 0; ExtentIndex < 2; ++ExtentIndex)
				{
					ParamValue[ExtentIndex] = (*UVs)[Index][ExtentIndex];
				}
				return ParamValue;
			};

			auto GetOrCreateRootUVStrandAttributeRef = [&]() 
			{
				TStrandAttributesRef<FVector2f> AttributeRootUV = HairDescription.StrandAttributes().GetAttributesRef<FVector2f>(HairAttribute::Strand::RootUV);
			
				// Add the RootUV attribute to the HairDescription if needed            
				if (!AttributeRootUV.IsValid())
				{
					HairDescription.StrandAttributes().RegisterAttribute<FVector2f>(HairAttribute::Strand::RootUV);
					AttributeRootUV = HairDescription.StrandAttributes().GetAttributesRef<FVector2f>(HairAttribute::Strand::RootUV);
				}
				return AttributeRootUV;                
			};

			Alembic::AbcGeom::GeometryScope UVScope = UVsParam.getScope();
			if (UVScope == Alembic::AbcGeom::kConstantScope || UVScope == Alembic::AbcGeom::kUniformScope)
			{
				TStrandAttributesRef<FVector2f> RootUVStrandAttributeRef = GetOrCreateRootUVStrandAttributeRef();
				for (uint32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
				{
					RootUVStrandAttributeRef[FStrandID(StartStrandID + CurveIndex)] = ConvertUVParam(CurveIndex);
				}
			}
			else if (UVScope == Alembic::AbcGeom::kVertexScope)
			{
				TStrandAttributesRef<FVector2f> RootUVStrandAttributeRef = GetOrCreateRootUVStrandAttributeRef();
				// RootUV is a strand attribute but UVsParam is vertex-scope so there's a NumVertices to NumCurves conversion needed
				int32 VertexIndex = 0;
				for (uint32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
				{
					RootUVStrandAttributeRef[FStrandID(StartStrandID + CurveIndex)] = ConvertUVParam(VertexIndex);

					// Increment by the number of vertices per curve
					VertexIndex += (*NumVertices)[CurveIndex];
				}
			}
		}

		// Set width values
		// Determine the scope of the WidthsParam
		Alembic::AbcGeom::GeometryScope WidthScope = Alembic::AbcGeom::kUnknownScope;
		Alembic::AbcGeom::IFloatGeomParam WidthParam = Curves.getSchema().getWidthsParam();
		if (WidthParam)
		{
			WidthScope = WidthParam.getScope();
		}

		if (WidthScope == Alembic::AbcGeom::kConstantScope || WidthScope == Alembic::AbcGeom::kUniformScope)
		{
			const float ConstWidth = (*Widths)[0] * Scale;
			TStrandAttributesRef<float> WidthStrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<float>(HairAttribute::Strand::Width);
			if (!WidthStrandAttributeRef.IsValid())
			{
				HairDescription.StrandAttributes().RegisterAttribute<float>(HairAttribute::Strand::Width);
				WidthStrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<float>(HairAttribute::Strand::Width);
			}

			for (uint32 Index = 0; Index < NumCurves; ++Index)
			{
				WidthStrandAttributeRef[FStrandID(StartStrandID + Index)] = WidthScope == Alembic::AbcGeom::kConstantScope ? ConstWidth : (*Widths)[Index] * Scale;
			}
		}
		else if (WidthScope == Alembic::AbcGeom::kVertexScope)
		{
			TVertexAttributesRef<float> VertexAttributeRef = HairDescription.VertexAttributes().GetAttributesRef<float>(HairAttribute::Vertex::Width);
			if (!VertexAttributeRef.IsValid())
			{
				HairDescription.VertexAttributes().RegisterAttribute<float>(HairAttribute::Vertex::Width);
				VertexAttributeRef = HairDescription.VertexAttributes().GetAttributesRef<float>(HairAttribute::Vertex::Width);
			}

			for (uint32 Index = 0; Index < NumPoints; ++Index)
			{
				VertexAttributeRef[FVertexID(StartVertexID + Index)] = (*Widths)[Index] * Scale;
			}
		}

		if (BasisType != Alembic::AbcGeom::kNoBasis && Knots)
		{
			TStrandAttributesRef<TArrayAttribute<float>> KnotsStrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<TArrayAttribute<float>>(HairAttribute::Strand::Knots);
			if (!KnotsStrandAttributeRef.IsValid())
			{
				HairDescription.StrandAttributes().RegisterAttribute<float[]>(HairAttribute::Strand::Knots);
				KnotsStrandAttributeRef = HairDescription.StrandAttributes().GetAttributesRef<TArrayAttribute<float>>(HairAttribute::Strand::Knots);
			}		

			uint32 GlobalKnotIndex = 0;
			for (uint32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
			{
				uint32 DegreeOfCurve = 1;
				if (CurveType == Alembic::AbcGeom::kCubic)
				{
					DegreeOfCurve = 3;
				}
				else if (CurveType == Alembic::AbcGeom::kVariableOrder)
				{
					if (ensureMsgf(Orders && CurveIndex < Orders->size(), TEXT("`Orders` index out of bounds, skipping knots for this strand.")))
					{
						DegreeOfCurve = (*Orders)[CurveIndex];
					}
					else
					{
						continue;
					}
				}

				TArrayAttribute<float> KnotsArrayAttribute = KnotsStrandAttributeRef[FStrandID(StartStrandID + CurveIndex)];

				uint32 CurveNumKnots = (*NumVertices)[CurveIndex] + DegreeOfCurve + 1;
				if (!ensureMsgf(GlobalKnotIndex + CurveNumKnots <= NumKnots, TEXT("Computed number of knots would exceed the number available from the groom.")))
				{
					CurveNumKnots = NumKnots - GlobalKnotIndex;
				}
				KnotsArrayAttribute.SetNum(CurveNumKnots);

				for (uint32 KnotIndex = 0; KnotIndex < CurveNumKnots; ++KnotIndex, ++GlobalKnotIndex)
				{
					KnotsArrayAttribute[KnotIndex] = (*Knots)[GlobalKnotIndex];
				}
			}

			ensureMsgf(GlobalKnotIndex == NumKnots, TEXT("Failed to translate all knots for the groom."));
		}

		// Extract the arbitrary GeomParams here, only need to do it once per ICurves
		Alembic::AbcGeom::ICompoundProperty ArbParams = Curves.getSchema().getArbGeomParams();
		if (ArbParams)
		{
			AlembicHairTranslatorUtils::ConvertAlembicAttributes(HairDescription, StartStrandID, NumCurves, StartVertexID, NumPoints, ArbParams, AnimInfo);
		}

		if (bCheckGroomAttributes)
		{
			// Groom attributes as UserProperties on ICurves, 1.3+
			Alembic::AbcGeom::ICompoundProperty Properties = Curves.getSchema().getUserProperties();
			if (Properties)
			{
				if (Properties.getNumProperties() > 0)
				{
					AlembicHairTranslatorUtils::SetGroomAttributes(HairDescription, Properties);
					bCheckGroomAttributes = false;
				}
			}
		}
	}
	else if (Alembic::AbcGeom::IXform::matches(ObjectMetaData))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ParseIXform);

		Alembic::AbcGeom::IXform Xform = Alembic::AbcGeom::IXform(InObject, Alembic::Abc::kWrapExisting);
		Alembic::AbcGeom::XformSample MatrixSample; 
		Xform.getSchema().get(MatrixSample);

		// The groom attributes should only be on the first IXform under the top node, no need to check for them once they are found
		if (bCheckGroomAttributes)
		{
			// Groom attributes as UserProperties, 1.1+
			Alembic::AbcGeom::ICompoundProperty Properties = Xform.getSchema().getUserProperties();
			if (Properties)
			{
				if (Properties.getNumProperties() > 0)
				{
					AlembicHairTranslatorUtils::SetGroomAttributes(HairDescription, Properties);
					bCheckGroomAttributes = false;
				}
			}

			// Groom attributes as GeomParams, as fallback to 1.0
			Alembic::AbcGeom::ICompoundProperty ArbParams = Xform.getSchema().getArbGeomParams();
			if (bCheckGroomAttributes && ArbParams)
			{
				if (ArbParams.getNumProperties() > 0)
				{
					AlembicHairTranslatorUtils::SetGroomAttributes(HairDescription, ArbParams);
					bCheckGroomAttributes = false;
				}
			}
		}

		LocalMatrix =  ParentMatrix * ConvertAlembicMatrix(MatrixSample.getMatrix());
	}

	if (NumChildren > 0)
	{
		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			ParseObject(InObject.getChild(ChildIndex), FrameTime, HairDescription, LocalMatrix, ConversionMatrix, Scale, bCheckGroomAttributes, AnimInfo);
		}
	}
}

bool FAlembicHairTranslator::Translate(const FString& FileName, FHairDescription& HairDescription, const FGroomConversionSettings& ConversionSettings)
{
	return Translate(FileName, HairDescription, ConversionSettings, nullptr);
}

bool FAlembicHairTranslator::Translate(const FString& FileName, FHairDescription& HairDescription, const struct FGroomConversionSettings& ConversionSettings, FGroomAnimationInfo* OutAnimInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAlembicHairTranslator::Translate);

	/** Factory used to generate objects*/
	Alembic::AbcCoreFactory::IFactory Factory;
	Alembic::AbcCoreFactory::IFactory::CoreType CompressionType = Alembic::AbcCoreFactory::IFactory::kUnknown;
	/** Archive-typed ABC file */
	Alembic::Abc::IArchive Archive;
	/** Alembic typed root (top) object*/
	Alembic::Abc::IObject TopObject;

	Factory.setPolicy(Alembic::Abc::ErrorHandler::kQuietNoopPolicy);
	Factory.setOgawaNumStreams(12);

	// Extract Archive and compression type from file
	Archive = Factory.getArchive(TCHAR_TO_UTF8(*FileName), CompressionType);
	if (!Archive.valid())
	{
		UE_LOGF(LogAlembicHairImporter, Warning, "Failed to open %ls: Not a valid Alembic file.", *FileName);
		return false;
	}

	// Get Top/root object
	TopObject = Alembic::Abc::IObject(Archive, Alembic::Abc::kTop);
	if (!TopObject.valid())
	{
		UE_LOGF(LogAlembicHairImporter, Warning, "Failed to import %ls: Root not is not valid.", *FileName);
		return false;
	}


	const int32 TimeSamplingIndex = Archive.getNumTimeSamplings() > 1 ? 1 : 0;
	Alembic::Abc::TimeSamplingPtr TimeSampler = Archive.getTimeSampling(TimeSamplingIndex);
	if (TimeSampler && OutAnimInfo)
	{
		OutAnimInfo->SecondsPerFrame = TimeSampler->getTimeSamplingType().getTimePerCycle();
	}

	FMatrix ConversionMatrix = FScaleMatrix::Make(ConversionSettings.Scale) * FRotationMatrix::Make(FQuat::MakeFromEuler(ConversionSettings.Rotation));
	FMatrix ParentMatrix = FMatrix::Identity;
	const float StrandsWidthScale = FMath::Abs(ConversionSettings.Scale.X); // Assume uniform scaling
	ParseObject(TopObject, 0.0f, HairDescription, ParentMatrix, ConversionMatrix, StrandsWidthScale, true, OutAnimInfo);

	return HairDescription.IsValid();
}

/** Private implementation to wrap Alembic Archive and its TopObject for parsing */
class FAbcPimpl
{
public:
	FAbcPimpl(const FString& FileName)
	{
		Alembic::AbcCoreFactory::IFactory::CoreType CompressionType = Alembic::AbcCoreFactory::IFactory::kUnknown;

		Factory.setPolicy(Alembic::Abc::ErrorHandler::kQuietNoopPolicy);
		Factory.setOgawaNumStreams(12);

		// Extract Archive and compression type from file
		Archive = Factory.getArchive(TCHAR_TO_UTF8(*FileName), CompressionType);
		if (!Archive.valid())
		{
			UE_LOGF(LogAlembicHairImporter, Warning, "Failed to open %ls: Not a valid Alembic file.", *FileName);
			return;
		}

		// Get Top/root object
		TopObject = Alembic::Abc::IObject(Archive, Alembic::Abc::kTop);
		if (!TopObject.valid())
		{
			UE_LOGF(LogAlembicHairImporter, Warning, "Failed to import %ls: Root not is not valid.", *FileName);
		}
	}

	bool IsValid() const
	{
		return TopObject.valid();
	}

	Alembic::AbcCoreFactory::IFactory Factory;
	Alembic::Abc::IArchive Archive;
	Alembic::Abc::IObject TopObject;
};

bool FAlembicHairTranslator::BeginTranslation(const FString& FileName)
{
	Abc = new FAbcPimpl(FileName);
	return Abc->IsValid();
}

bool FAlembicHairTranslator::Translate(float FrameTime, FHairDescription& HairDescription, const struct FGroomConversionSettings& ConversionSettings)
{
	if (!Abc || !Abc->IsValid())
	{
		return false;
	}

	FMatrix ConversionMatrix = FScaleMatrix::Make(ConversionSettings.Scale) * FRotationMatrix::Make(FQuat::MakeFromEuler(ConversionSettings.Rotation));
	FMatrix ParentMatrix = FMatrix::Identity;
	const float StrandsWidthScale = FMath::Abs(ConversionSettings.Scale.X); // Assume uniform scaling
	ParseObject(Abc->TopObject, FrameTime, HairDescription, ParentMatrix, ConversionMatrix, StrandsWidthScale, true, nullptr);

	return HairDescription.IsValid();
}

void FAlembicHairTranslator::EndTranslation()
{
	delete Abc;
	Abc = nullptr;
}

FAlembicHairTranslator::FAlembicHairTranslator()
: Abc(nullptr)
{
}


FAlembicHairTranslator::~FAlembicHairTranslator()
{
	EndTranslation();
}

static void ValidateObject(const Alembic::Abc::IObject& InObject, bool& bHasGeometry, int32& NumCurves)
{
	if (!InObject)
	{
		return;
	}

	// Validate that the Alembic has curves only
	// Any PolyMesh will cause the Alembic to be rejected by this translator

	Alembic::AbcCoreAbstract::ObjectHeader Header = InObject.getHeader();
	const Alembic::Abc::MetaData ObjectMetaData = InObject.getMetaData();
	const uint32 NumChildren = InObject.getNumChildren();

	if (Alembic::AbcGeom::ICurves::matches(ObjectMetaData))
	{
		++NumCurves;
	}
	else if (Alembic::AbcGeom::IPolyMesh::matches(ObjectMetaData))
	{
		bHasGeometry = true;
	}

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren && !bHasGeometry; ++ChildIndex)
	{
		ValidateObject(InObject.getChild(ChildIndex), bHasGeometry, NumCurves);
	}
}

bool FAlembicHairTranslator::CanTranslate(const FString& FilePath)
{
	if (!IsFileExtensionSupported(FPaths::GetExtension(FilePath)))
	{
		return false;
	}

	/** Factory used to generate objects*/
	Alembic::AbcCoreFactory::IFactory Factory;
	Alembic::AbcCoreFactory::IFactory::CoreType CompressionType = Alembic::AbcCoreFactory::IFactory::kUnknown;
	/** Archive-typed ABC file */
	Alembic::Abc::IArchive Archive;
	/** Alembic typed root (top) object*/
	Alembic::Abc::IObject TopObject;

	Factory.setPolicy(Alembic::Abc::ErrorHandler::kQuietNoopPolicy);
	Factory.setOgawaNumStreams(12);

	// Extract Archive and compression type from file
	Archive = Factory.getArchive(TCHAR_TO_UTF8(*FilePath), CompressionType);
	if (!Archive.valid())
	{
		return false;
	}

	// Get Top/root object
	TopObject = Alembic::Abc::IObject(Archive, Alembic::Abc::kTop);
	if (!TopObject.valid())
	{
		return false;
	}

	bool bHasGeometry = false;
	int32 NumCurves = 0;

	ValidateObject(TopObject, bHasGeometry, NumCurves);

	return !bHasGeometry && NumCurves > 0;
}

bool FAlembicHairTranslator::IsFileExtensionSupported(const FString& FileExtension) const
{
	return GetSupportedFormat().StartsWith(FileExtension);
}

FString FAlembicHairTranslator::GetSupportedFormat() const
{
	return TEXT("abc;Alembic hair strands file");
}
