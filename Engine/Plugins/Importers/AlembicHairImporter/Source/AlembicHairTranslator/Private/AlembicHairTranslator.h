// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HairStrandsTranslator.h"

struct FGroomAnimationInfo;
struct FGroomConversionSettings;

class FAlembicHairTranslator : public IGroomTranslator
{
public:
	FAlembicHairTranslator();
	virtual ~FAlembicHairTranslator();

	virtual bool Translate(const FString& FilePath, FHairDescription& OutHairDescription, const FGroomConversionSettings& ConversionSettings) override;
	virtual bool CanTranslate(const FString& FilePath) override;
	virtual bool IsFileExtensionSupported(const FString& FileExtension) const override;
	virtual FString GetSupportedFormat() const override;

	virtual bool Translate(const FString& FilePath, FHairDescription& OutHairDescription, const struct FGroomConversionSettings& ConversionSettings, FGroomAnimationInfo* OutAnimInfo) override;
	virtual bool BeginTranslation(const FString& FilePath) override;
	virtual bool Translate(float FrameTime, FHairDescription& OutHairDescription, const struct FGroomConversionSettings& ConversionSettings) override;
	virtual void EndTranslation() override;

private:
	class FAbcPimpl* Abc;
};
