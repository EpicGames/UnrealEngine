// Copyright Epic Games, Inc. All Rights Reserved.

#include "AlembicHairTranslatorModule.h"

#include "AlembicHairTranslator.h"
#include "HairStrandsEditor.h"

LLM_DECLARE_TAG_API(GroomEditor, HAIRSTRANDSEDITOR_API);

IMPLEMENT_MODULE(FAlembicHairTranslatorModule, AlembicHairTranslatorModule);

void FAlembicHairTranslatorModule::StartupModule()
{
	LLM_SCOPE_BYTAG(GroomEditor)
	FGroomEditor::Get().RegisterHairTranslator<FAlembicHairTranslator>();
}

void FAlembicHairTranslatorModule::ShutdownModule()
{
}

